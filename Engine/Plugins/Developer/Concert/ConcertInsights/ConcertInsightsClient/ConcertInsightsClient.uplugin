{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "ConcertInsightsClient",
	"Description": "Extends status bar so you can start a synchronized trace on all connected Concert endpoints.",
	"Category": "Networking",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"CanContainContent": true,
	"Hidden": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ConcertInsightsClient",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "ConcertInsightsCore",
			"Enabled": true
		},
		{
			"Name": "ConcertSharedSlate",
			"Enabled": true
		},
		{
			"Name": "ConcertSyncClient",
			"Enabled": true
		},
		{
			"Name": "TraceUtilities",
			"Enabled": true
		}
	]
}