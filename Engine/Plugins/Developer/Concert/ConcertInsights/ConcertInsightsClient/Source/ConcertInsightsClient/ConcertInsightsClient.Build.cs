// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ConcertInsightsClient : ModuleRules
{
    public ConcertInsightsClient(ReadOnlyTargetRules Target) : base(Target)
    {
        PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

        PublicDependencyModuleNames.AddRange(
            new string[]
            {
                "Core",
            }
        );

        PrivateDependencyModuleNames.AddRange(
            new string[]
            {
	            "ConcertInsightsCore",
	            "ConcertSharedSlate",
	            "ConcertSyncClient",
                "CoreUObject",
                "EditorTraceUtilities",
                "Engine",
                "Slate",
                "SlateCore",
                "ToolMenus", 
            }
        );
        
        ShortName = "CrtInEd";
    }
}