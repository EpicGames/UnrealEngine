// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClientTraceControls.h"

#include "ConcertInsightsClientSettings.h"
#include "IConcertSyncClient.h"
#include "IConcertSyncClientModule.h"

#include "IEditorTraceUtilitiesModule.h"
#include "ProfilingDebugging/TraceAuxiliary.h"

namespace UE::ConcertInsightsClient
{
	FClientTraceControls::FClientTraceControls()
	{
		if (const TSharedPtr<IConcertSyncClient> Client = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
		{
			Client->GetConcertClient()->OnSessionStartup().AddRaw(this, &FClientTraceControls::OnSessionStart);
			Client->GetConcertClient()->OnSessionShutdown().AddRaw(this, &FClientTraceControls::OnSessionStopped);
		}
	}

	FClientTraceControls::~FClientTraceControls()
	{
		if (!IConcertSyncClientModule::IsAvailable())
		{
			return;
		}
		
		if (const TSharedPtr<IConcertSyncClient> Client = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser")))
		{
			Client->GetConcertClient()->OnSessionStartup().RemoveAll(this);
		}
		
		// ~FTraceControls will clean up delegates of registered sessions itself
	}

	ConcertInsightsCore::FStartTraceArgs FClientTraceControls::GetDefaultSynchronizedTraceArgs() const
	{
		EditorTraceUtilities::IEditorTraceUtilitiesModule& Module = EditorTraceUtilities::IEditorTraceUtilitiesModule::Get();
		const EditorTraceUtilities::FStatusBarTraceSettings& Settings = Module.GetTraceSettings();

		ConcertInsightsCore::FStartTraceArgs Result;
		switch (Settings.TraceDestination)
		{
		case EditorTraceUtilities::ETraceDestination::TraceStore:
			Result.ConnectionType = EConcertTraceTargetType::Network;
			// TODO DP: Figure out the IP address of the trace store since localhost will mean something different for all the other machines
			Result.Target = UConcertInsightsClientSettings::Get()->SynchronizedTraceDestinationIP;
			break;
		case EditorTraceUtilities::ETraceDestination::File:
			Result.ConnectionType = EConcertTraceTargetType::File;
			Result.Target = nullptr;
			break;
		default:
			checkNoEntry();
		}
		
		return Result;
	}

	bool FClientTraceControls::CanSendRequestsToEndpoint(const FGuid& EndpointId, const IConcertSession& Session) const
	{
		// Do not send to own endpoint ID - it does not break anything but there's no point.
		const TSharedPtr<IConcertSyncClient> Client = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
		const TSharedPtr<IConcertClientSession> ClientSession = Client->GetConcertClient()->GetCurrentSession();
		return !ClientSession || EndpointId != ClientSession->GetSessionClientEndpointId();
	}

	ConcertInsightsCore::FInitArgs FClientTraceControls::GetInitEventArgs() const
	{
		const TSharedPtr<IConcertSyncClient> Client = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
		const TSharedPtr<IConcertClientSession> Session = Client->GetConcertClient()->GetCurrentSession();
		if (!Session)
		{
			return {};
		}
		
		return { Session->GetSessionClientEndpointId(), Session->GetLocalClientInfo().DisplayName, false };
	}

	void FClientTraceControls::OnSessionStart(TSharedRef<IConcertClientSession> Session)
	{
		RegisterTraceRequestsHandler(Session);
	}

	void FClientTraceControls::OnSessionStopped(TSharedRef<IConcertClientSession> Session)
	{
		OnLeaveSession(*Session);
	}
}
