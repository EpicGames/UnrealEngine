// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IConcertSyncClient.h"
#include "TraceControls.h"

namespace UE::ConcertInsightsClient
{
	/**
	 * Client controls for the editor.
	 * Puts the client's session and display name into the init args.
	 */
	class FClientTraceControls : public ConcertInsightsCore::FTraceControls
	{
		friend class FTraceControls; // For FTraceControls::Make
	public:
		
		virtual ~FClientTraceControls() override;
		
	protected:
		
		//~ Begin FTraceControls Interface
		virtual ConcertInsightsCore::FStartTraceArgs GetDefaultSynchronizedTraceArgs() const override;
		virtual bool CanSendRequestsToEndpoint(const FGuid& EndpointId, const IConcertSession& Session) const override;
		virtual ConcertInsightsCore::FInitArgs GetInitEventArgs() const override;
		//~ End FTraceControls Interface

	private:
		
		FClientTraceControls();
		
		void OnSessionStart(TSharedRef<IConcertClientSession> Session);
		void OnSessionStopped(TSharedRef<IConcertClientSession> Session);
	};
}
