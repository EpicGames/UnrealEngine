// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConcertInsightsClient.h"

#include "StatusBar/SMultiUserStatusBar.h"
#include "StatusBar/StatusBarExtension.h"

#include "Application/SlateApplicationBase.h"
#include "Misc/CoreDelegates.h"

#define LOCTEXT_NAMESPACE "FConcertInsightsEditorModule"

namespace UE::ConcertInsightsClient
{
	void FConcertInsightsClientModule::StartupModule()
	{
		FCoreDelegates::GetOnPostEngineInit().AddRaw(this, &FConcertInsightsClientModule::PostEngineInit);
	}

	void FConcertInsightsClientModule::ShutdownModule()
	{}

	void FConcertInsightsClientModule::PostEngineInit()
	{
		if (FSlateApplicationBase::IsInitialized())
		{
			// Multi User does not add any status bar by itself. For now, we'll add one but in the future we may want this to be in the MU module.
			ExtendEditorStatusBarWithMultiUserWidget();
		
			TraceControls = ConcertInsightsCore::FTraceControls::Make<FClientTraceControls>();
			ExtendMultiUserStatusBarWithInsights(*TraceControls);
		}
	}
}


#undef LOCTEXT_NAMESPACE
    
IMPLEMENT_MODULE(UE::ConcertInsightsClient::FConcertInsightsClientModule, ConcertInsightsClient)