// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClientTraceControls.h"
#include "IConcertInsightsClientModule.h"
#include "Templates/UniquePtr.h"

namespace UE::ConcertInsightsClient
{
	class FConcertInsightsClientModule : public IConcertInsightsClientModule
	{
	public:
		
		//~ Begin IModuleInterface Interface
		virtual void StartupModule() override;
		virtual void ShutdownModule() override;
		//~ End IModuleInterface Interface

	private:

		/** The local state of synchronized tracing. */
		TUniquePtr<FClientTraceControls> TraceControls;

		void PostEngineInit();
	};
}
