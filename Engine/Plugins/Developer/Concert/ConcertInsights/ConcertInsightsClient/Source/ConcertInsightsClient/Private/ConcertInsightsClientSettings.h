// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "ConcertInsightsClientSettings.generated.h"

UCLASS(config = EditorPerProjectUserSettings)
class CONCERTINSIGHTSCLIENT_API UConcertInsightsClientSettings : public UObject
{
	GENERATED_BODY()
public:

	static UConcertInsightsClientSettings* Get() { return GetMutableDefault<UConcertInsightsClientSettings>(); }

	/** This is the IP address that is sent to all endpoints to trace to when starting a synchronized trace. */
	UPROPERTY(config)
	FString SynchronizedTraceDestinationIP = TEXT("localhost");
};
