// Copyright Epic Games, Inc. All Rights Reserved.

#include "SEditTraceDestinationWidget.h"

#include "ConcertInsightsClientSettings.h"
#include "Internationalization/Regex.h"
#include "Widgets/Input/SEditableTextBox.h"

#define LOCTEXT_NAMESPACE "SEditTraceDestinationWidget"

namespace UE::ConcertInsightsClient
{
	void SEditTraceDestinationWidget::Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			SNew(SEditableTextBox)
			.MinDesiredWidth(100.f)
			.Text_Lambda([](){ return FText::FromString(UConcertInsightsClientSettings::Get()->SynchronizedTraceDestinationIP); })
			.OnTextCommitted_Lambda([](const FText& Text, ETextCommit::Type)
			{
				UConcertInsightsClientSettings* Settings = UConcertInsightsClientSettings::Get();
				const FString TextString = Text.ToString();
				if (Settings->SynchronizedTraceDestinationIP != TextString)
				{
					Settings->SynchronizedTraceDestinationIP = Text.ToString();
					Settings->SaveConfig();
				}
			})
			.OnVerifyTextChanged_Lambda([](const FText& TextToVerify, FText& Error)
			{
				const FString TextString = TextToVerify.ToString();
				if (TextString.Equals(TEXT("localhost"), ESearchCase::IgnoreCase))
				{
					return true;
				}
				
				// Regex for matching ipv4 address stolen from https://www.oreilly.com/library/view/regular-expressions-cookbook/9780596802837/ch07s16.html
				const FRegexPattern Pattern(TEXT("^(?:[0-9]{1,3}\\.){3}[0-9]{1,3}$"));
				FRegexMatcher Matcher(Pattern, TextString);
				
				const bool bIsIp = Matcher.FindNext();
				if (!bIsIp)
				{
					Error = LOCTEXT("InvalidIp", "Invalid IPv4 Address");
				}
				return bIsIp;
			})
		];
	}
}

#undef LOCTEXT_NAMESPACE