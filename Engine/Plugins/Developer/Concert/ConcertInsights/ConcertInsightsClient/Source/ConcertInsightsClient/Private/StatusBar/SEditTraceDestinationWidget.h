// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::ConcertInsightsClient
{
	class SEditTraceDestinationWidget : public SCompoundWidget
	{
	public:

		SLATE_BEGIN_ARGS(SEditTraceDestinationWidget){}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);
	};
}


