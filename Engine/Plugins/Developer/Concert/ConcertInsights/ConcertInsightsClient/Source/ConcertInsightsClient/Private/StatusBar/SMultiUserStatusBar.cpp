// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMultiUserStatusBar.h"

#include "ConcertFrontendStyle.h"
#include "IConcertSyncClient.h"
#include "IConcertSyncClientModule.h"

#include "Framework/Docking/TabManager.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"
#include "ToolMenus.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SScaleBox.h"

#define LOCTEXT_NAMESPACE "SMultiUserStatusBar"

namespace UE::ConcertInsightsClient
{
	void ExtendEditorStatusBarWithMultiUserWidget()
	{
		SMultiUserStatusBar::RegisterMultiUserToolMenu();
		
		UToolMenu* Menu = UToolMenus::Get()->ExtendMenu(TEXT("LevelEditor.StatusBar.ToolBar"));
		FToolMenuSection& MultiUserSection = Menu->AddSection(TEXT("MultiUser"), FText::GetEmpty(), FToolMenuInsert(NAME_None, EToolMenuInsertType::First));
		MultiUserSection.AddEntry(
			FToolMenuEntry::InitWidget(TEXT("MultiUserStatusBar"), SNew(SMultiUserStatusBar), FText::GetEmpty(), true, false)
		);
	}

	void SMultiUserStatusBar::RegisterMultiUserToolMenu()
	{
		UToolMenus* ToolMenus = UToolMenus::Get();
		if (!ToolMenus->IsMenuRegistered(TEXT("MultiUser.StatusBarMenu")))
		{
			ToolMenus->RegisterMenu(TEXT("MultiUser.StatusBarMenu"));
			// Could add additional utilities here, like opening the MU tab
		}
	}

	void SMultiUserStatusBar::Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			SNew(SHorizontalBox)
			.ToolTipText_Static(&SMultiUserStatusBar::GetConnectionTooltip)
			
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				MakeComboButton()
			]
			
			+SHorizontalBox::Slot()
			.AutoWidth()
			[
				MakeSessionConnectionIndicator()
			]
		];
	}

	TSharedRef<SWidget> SMultiUserStatusBar::MakeComboButton()
	{
		return SNew(SComboButton)
			.ContentPadding(FMargin(6.0f, 0.0f))
			.MenuPlacement(MenuPlacement_AboveAnchor)
			.ComboButtonStyle(&FAppStyle::Get().GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
			.OnGetMenuContent(this, &SMultiUserStatusBar::MakeTraceMenu)
			.HasDownArrow(true)
			.ButtonContent()
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(0, 0, 3, 0)
				[
					SNew(SImage)
					.ColorAndOpacity(FSlateColor::UseForeground())
					.Image(FConcertFrontendStyle::Get()->GetBrush(TEXT("Concert.MultiUser")))
				]

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				// Last horizontal box should pad to the right so our status bar entry looks consistent with all the other status bar entries
				.Padding(0, 0, 2, 0)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("MultiUser", "Multi User"))
					.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("DialogButtonText"))
				]
			];
	}

	TSharedRef<SWidget> SMultiUserStatusBar::MakeSessionConnectionIndicator() const
	{
		return SNew(SHorizontalBox)
		
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(2.f)
			[
				SNew(SScaleBox)
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush("Icons.FilledCircle"))
					.ColorAndOpacity_Lambda([this]()
					{
						return IsConnectedToSession()
							? FStyleColors::AccentGreen
							: FStyleColors::AccentGray;
					})
				]
			]
		
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Text_Static(&SMultiUserStatusBar::GetCurrentSessionLabel)
			]
		;
	}

	TSharedRef<SWidget> SMultiUserStatusBar::MakeTraceMenu() const
	{
		UToolMenus* ToolMenus = UToolMenus::Get();
		if (!ToolMenus->IsMenuRegistered(TEXT("MultiUser.StatusBarMenu")))
		{
			ToolMenus->RegisterMenu(TEXT("MultiUser.StatusBarMenu"));
			// Could add additional utilities here, like opening the MU tab
		}

		FToolMenuContext Context;
		UToolMenu* Menu = UToolMenus::Get()->GenerateMenu(TEXT("MultiUser.StatusBarMenu"), Context);
		return ToolMenus->GenerateWidget(Menu);
	}

	bool SMultiUserStatusBar::IsConnectedToSession()
	{
		const TSharedPtr<IConcertSyncClient> Client = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
		return Client && Client->GetConcertClient()->GetCurrentSession().IsValid();
	}

	FText SMultiUserStatusBar::GetConnectionTooltip()
	{
		const TSharedPtr<IConcertSyncClient> Client = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
		const TSharedPtr<IConcertClientSession> Session = Client->GetConcertClient()->GetCurrentSession();
		const bool bIsConnected = Session.IsValid();
		return bIsConnected
			? FText::Format(LOCTEXT("ConnectionIndicator.OnlineFmt", "Connected to session {0}"), FText::FromString(Session->GetSessionInfo().SessionName))
			: LOCTEXT("ConnectionIndicator.Offline", "Disconnected");
	}

	FText SMultiUserStatusBar::GetCurrentSessionLabel()
	{
		const TSharedPtr<IConcertSyncClient> Client = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
		const TSharedPtr<IConcertClientSession> Session = Client->GetConcertClient()->GetCurrentSession();
		const bool bIsConnected = Session.IsValid();
		return bIsConnected
			? FText::FromString(Session->GetSessionInfo().SessionName)
			: LOCTEXT("CurrentSession.None", "No session");
	}
}

#undef LOCTEXT_NAMESPACE