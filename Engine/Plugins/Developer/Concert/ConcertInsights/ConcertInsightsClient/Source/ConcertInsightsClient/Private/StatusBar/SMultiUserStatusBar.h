// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::ConcertInsightsClient
{
	/** Extends the editor status bar with the Multi-User widget. */
	void ExtendEditorStatusBarWithMultiUserWidget();
	
	class SMultiUserStatusBar : public SCompoundWidget
	{
	public:

		/** Register the UToolMenu for the combo button. */
		static void RegisterMultiUserToolMenu();

		SLATE_BEGIN_ARGS(SMultiUserStatusBar) {}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs);

	private:
		TSharedRef<SWidget> MakeComboButton();
		TSharedRef<SWidget> MakeSessionConnectionIndicator() const;
		
		TSharedRef<SWidget> MakeTraceMenu() const;

		static bool IsConnectedToSession();
		static FText GetConnectionTooltip();
		static FText GetCurrentSessionLabel();
	};
}

