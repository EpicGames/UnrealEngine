// Copyright Epic Games, Inc. All Rights Reserved.

#include "StatusBarExtension.h"

#include "ConcertInsightsClient.h"
#include "IConcertSyncClient.h"
#include "IConcertSyncClientModule.h"
#include "SEditTraceDestinationWidget.h"

#include "ToolMenus.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "MultiUserStatusBarExtension"

namespace UE::ConcertInsightsClient
{
	namespace Private
	{
		static bool IsInSession()
		{
			const TSharedPtr<IConcertSyncClient> Client = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
			const TSharedPtr<IConcertClientSession> Session = Client->GetConcertClient()->GetCurrentSession();
			return Session.IsValid();
		}

		static FToolMenuEntry MakeSynchronizedTraceEntry(FClientTraceControls& Controls)
		{
			FToolMenuEntry ConcertTraceEntry = FToolMenuEntry::InitMenuEntry(
				TEXT("StartConcertTrace"),
				TAttribute<FText>::CreateLambda([&Controls]()
				{
					return Controls.IsTracing()
						? LOCTEXT("Menu.Tracing.ToggleSynchronizedTrace.StopLabel", "Stop synchronized trace")
						: LOCTEXT("Menu.Tracing.ToggleSynchronizedTrace.StartLabel", "Start synchronized trace");
				}),
				TAttribute<FText>::CreateLambda([&Controls]()
				{
					if (!Private::IsInSession())
					{
						return LOCTEXT("Menu.Tracing.ToggleSynchronizedTrace.Tooltip.NotInSession", "Not in any Multi-User session. Join a session first.");
					}
					
					return Controls.IsTracing()
						? LOCTEXT("Menu.Tracing.ToggleSynchronizedTrace.Tooltip.StopTrace", "Stops the synchronized trace across all participants")
						: LOCTEXT("Menu.Tracing.ToggleSynchronizedTrace.Tooltip.StartTrace", "Starts a synchronized trace across endpoints in the current Multi-User session.");
				}),
				FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateLambda([&Controls]()
					{
						const TSharedPtr<IConcertSyncClient> Client = IConcertSyncClientModule::Get().GetClient(TEXT("MultiUser"));
						const TSharedPtr<IConcertClientSession> Session = Client->GetConcertClient()->GetCurrentSession();
						if (!Session)
						{
							return;
						}
						
						if (Controls.IsTracing())
						{
							Controls.StopSynchronizedTrace();
						}
						else
						{
							FText ErrorReason;
							const bool bSuccess = Controls.StartSynchronizedTrace(Session.ToSharedRef(), &ErrorReason);
							if (!bSuccess)
							{
								FNotificationInfo Notification(LOCTEXT("NotificationTitle", "Synchronized Trace Failed"));
								Notification.SubText = ErrorReason;
								Notification.bFireAndForget = true;
								Notification.ExpireDuration = 4.f;
								FSlateNotificationManager::Get().AddNotification(Notification)
									->SetCompletionState(SNotificationItem::CS_Fail);
							}
						}
					}),
					FCanExecuteAction::CreateLambda([]() { return Private::IsInSession();})
					)
			);
			ConcertTraceEntry.bShouldCloseWindowAfterMenuSelection = false;

			return ConcertTraceEntry;
		}
		
		static FToolMenuEntry MakeTraceDestinationIpEntry()
		{
			return FToolMenuEntry::InitWidget(
				TEXT("EditIp"),
				SNew(SEditTraceDestinationWidget),
				LOCTEXT("EditIp.Label", "Destination IP"),
				true, true, false,
				LOCTEXT("EditIp.ToolTip", "Enter the IP of the trace store to send all trace data to. Tracing will fail to start if the IP is invalid.")
				);
		}
	}
	
	void ExtendMultiUserStatusBarWithInsights(FClientTraceControls& Controls)
	{
		UToolMenu* ToolMenu = UToolMenus::Get()->ExtendMenu(TEXT("MultiUser.StatusBarMenu"));
		if (!ToolMenu)
		{
			return;
		}

		FToolMenuSection& ControlsSection = ToolMenu->AddSection(TEXT("Tracing"), LOCTEXT("Menu.Tracing", "Tracing"));
		ControlsSection.AddEntry(Private::MakeSynchronizedTraceEntry(Controls));
		ControlsSection.AddEntry(Private::MakeTraceDestinationIpEntry());
	}
}

#undef LOCTEXT_NAMESPACE