// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

namespace UE::ConcertInsightsClient
{
	class FClientTraceControls;
	
	/** Adds a sub-menu to the Multi User status bar. */
	void ExtendMultiUserStatusBarWithInsights(FClientTraceControls& Controls);
}

