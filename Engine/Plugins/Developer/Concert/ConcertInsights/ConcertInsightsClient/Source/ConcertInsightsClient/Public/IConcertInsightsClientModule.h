// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"

namespace UE::ConcertInsightsClient
{
	/**
	 * On the editor, this module
	 * - adds a context menu option to the Unreal Insights editor menu (bottom-right) for starting synchronized session tracing across multiple machines
	 * - listens for requests to synchronized session tracing while the local editor is in a Concert session
	 */
	class CONCERTINSIGHTSCLIENT_API IConcertInsightsClientModule : public IModuleInterface
	{
	public:

		/**
		 * Singleton-like access to this module's interface.  This is just for convenience!
		 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
		 *
		 * @return Returns singleton instance, loading the module on demand if needed
		 */
		static inline IConcertInsightsClientModule& Get()
		{
			static const FName ModuleName = "ConcertInsightsClient";
			return FModuleManager::LoadModuleChecked<IConcertInsightsClientModule>(ModuleName);
		}

		/**
		 * Checks to see if this module is loaded and ready.  It is only valid to call Get() during shutdown if IsAvailable() returns true.
		 *
		 * @return True if the module is loaded and ready to use
		 */
		static inline bool IsAvailable()
		{
			static const FName ModuleName = "ConcertInsightsClient";
			return FModuleManager::Get().IsModuleLoaded(ModuleName);
		}
	};
}

